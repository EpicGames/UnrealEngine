{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "0.1",
	"FriendlyName" : "CharacterAI",
	"Description" : "Adds code and assets related to implementing AI in a character-based project.",
	"Category" : "Gameplay",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : true,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "CharacterAI",
			"Type" : "Runtime",
			"LoadingPhase" : "PostEngineInit"
		}
	]
}