// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CharacterAI : ModuleRules
{
    public CharacterAI(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"RenderCore",
				"RHI",
                "AIModule",
			}
			);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Slate",
				"Renderer",
			}
			);

		if (Target.bBuildEditor == true)
		{
			//@TODO: Needed for the triangulation code used for sprites (but only in editor mode)
			//@TOOD: Try to move the code dependent on the triangulation code to the editor-only module
			PrivateDependencyModuleNames.Add("EditorFramework");
			PrivateDependencyModuleNames.Add("UnrealEd");
		}
	}
}
