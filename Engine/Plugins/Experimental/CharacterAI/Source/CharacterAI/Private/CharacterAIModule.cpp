// Copyright Epic Games, Inc. All Rights Reserved.

#include "CharacterAIModule.h"
#include "Modules/ModuleManager.h"
#include "CharacterAIPrivate.h"

//////////////////////////////////////////////////////////////////////////
// FCharacterAIModule

class FCharacterAIModule : public ICharacterAIModuleInterface
{
public:
	virtual void StartupModule() override
	{
		check(GConfig);		
	}

	virtual void ShutdownModule() override
	{
	}
};

//////////////////////////////////////////////////////////////////////////

IMPLEMENT_MODULE(FCharacterAIModule, CharacterAI);
DEFINE_LOG_CATEGORY(LogCharacterAI);
