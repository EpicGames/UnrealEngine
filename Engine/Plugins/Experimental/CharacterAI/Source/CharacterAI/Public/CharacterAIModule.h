// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

#define CHARACTERAI_MODULE_NAME "CharacterAI"

//////////////////////////////////////////////////////////////////////////
// ICharacterAIModuleInterface

class ICharacterAIModuleInterface : public IModuleInterface
{
};

